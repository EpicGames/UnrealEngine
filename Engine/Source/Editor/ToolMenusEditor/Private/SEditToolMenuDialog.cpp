// Copyright Epic Games, Inc. All Rights Reserved.


#include "SEditToolMenuDialog.h"
#include "IToolMenusEditorModule.h"

#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "IDetailsView.h"
#include "Misc/App.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SScrollBox.h"

#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"

#include "ToolMenus.h"

#include "Editor.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Serialization/StaticMemoryReader.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Framework/Commands/UICommandDragDropOp.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "EditToolMenuDialog"

class SVisibilityEditWidget : public SImage
{
public:
	SLATE_BEGIN_ARGS(SVisibilityEditWidget) {}
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs, TSharedRef<SMultiBoxWidget> InBaseWidget, TSharedRef<const FMultiBlock> InBlock, TSharedRef<SEditToolMenuDialog> InDialogWidget)
	{
		BaseWidget = InBaseWidget;
		Block = InBlock;
		DialogWidget = InDialogWidget;

		SImage::Construct(
			SImage::FArguments()
			.Image(this, &SVisibilityEditWidget::GetBrush)
		);
	}

private:

	FReply HandleClick()
	{
		ToggleVisible();
		return FReply::Handled();
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		return HandleClick();
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
		{
			return FReply::Unhandled();
		}

		return HandleClick();
	}

	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	const FSlateBrush* GetBrush() const
	{
		if (IsVisible())
		{
			static const FName NAME_VisibleHoveredBrush("Level.VisibleHighlightIcon16x");
			static const FName NAME_VisibleNotHoveredBrush("Level.VisibleIcon16x");
			return IsHovered() ? FAppStyle::GetBrush(NAME_VisibleHoveredBrush) :
				FAppStyle::GetBrush(NAME_VisibleNotHoveredBrush);
		}
		else
		{
			static const FName NAME_NotVisibleHoveredBrush("Level.NotVisibleHighlightIcon16x");
			static const FName NAME_NotVisibleNotHoveredBrush("Level.NotVisibleIcon16x");
			return IsHovered() ? FAppStyle::GetBrush(NAME_NotVisibleHoveredBrush) :
				FAppStyle::GetBrush(NAME_NotVisibleNotHoveredBrush);
		}
	}

	bool IsVisible() const
	{
		const FName ItemName = Block->GetExtensionHook();
		if (ItemName == NAME_None)
		{
			return true;
		}

		UToolMenuBase* ToolMenu = BaseWidget.IsValid() ? BaseWidget.Pin()->GetMultiBox()->GetToolMenu() : nullptr;
		if (!ToolMenu)
		{
			return true;
		}

		FCustomizedToolMenuHierarchy MenuCustomizationHierarchy = ToolMenu->GetMenuCustomizationHierarchy();
		if (MenuCustomizationHierarchy.Hierarchy.Num() > 0)
		{
			if (Block->GetType() == EMultiBlockType::Heading)
			{
				return !MenuCustomizationHierarchy.IsSectionHidden(ItemName);
			}
			else
			{
				return !MenuCustomizationHierarchy.IsEntryHidden(ItemName);
			}
		}

		return true;
	}	
	
	void ToggleVisible()
	{
		if (DialogWidget.IsValid())
		{
			DialogWidget.Pin()->OnToggleVisibleClicked(Block.ToSharedRef(), BaseWidget);
		}
	}

	TSharedPtr<const FMultiBlock> Block;
	TWeakPtr<SMultiBoxWidget> BaseWidget;
	TWeakPtr<SEditToolMenuDialog> DialogWidget;
};

class SMultiBlockDragHandle : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SMultiBlockDragHandle ) {}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, TSharedRef<SMultiBoxWidget> InBaseWidget, TSharedRef<const FMultiBlock> InBlock, bool bInIsBlockNamedAndToolMenu );

	FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	
		bool bIsDropDestination = false;
		bool bInsertAfter = false;
		if (BaseWidget.IsValid() && Block.IsValid())
		{
			const EVisibility Status = BaseWidget.Pin()->GetCustomizationBorderDragVisibility(Block->GetExtensionHook(), Block->GetType(), bInsertAfter);
			if (Status == EVisibility::Visible)
			{
				bIsDropDestination = true;
			}
		}

		if (bIsDropDestination)
		{
			const FSlateBrush* DropIndicatorBrush = FAppStyle::GetBrush(bInsertAfter ? "MultiBox.DragBelow" : "MultiBox.DragAbove");

			FSlateDrawElement::MakeBox
			(
				OutDrawElements,
				LayerId++,
				AllottedGeometry.ToPaintGeometry(),
				DropIndicatorBrush,
				ESlateDrawEffect::None,
				DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
			);
		}

		return LayerId;
	}

private:

	TSharedPtr<const FMultiBlock> Block;
	TWeakPtr<SMultiBoxWidget> BaseWidget;
	bool bIsBlockNamedAndToolMenu;
};


void SMultiBlockDragHandle::Construct( const FArguments& InArgs, TSharedRef<SMultiBoxWidget> InBaseWidget, TSharedRef<const FMultiBlock> InBlock, bool bInIsBlockNamedAndToolMenu )
{
	BaseWidget = InBaseWidget;
	Block = InBlock;
	bIsBlockNamedAndToolMenu = bInIsBlockNamedAndToolMenu;
}

FReply SMultiBlockDragHandle::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (Block->GetExtensionHook() != NAME_None && bIsBlockNamedAndToolMenu)
		{
			return FReply::Handled().DetectDrag(SharedThis(this), MouseEvent.GetEffectingButton());
		}
	}

	return FReply::Unhandled();
}

FReply SMultiBlockDragHandle::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		BaseWidget.Pin()->GetMultiBox()->OnEditSelectionChanged().ExecuteIfBound(Block.ToSharedRef());
	}

	return FReply::Unhandled();
}

void SMultiBlockDragHandle::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FUICommandDragDropOp>().IsValid() )
	{
		BaseWidget.Pin()->OnCustomCommandDragEnter( Block.ToSharedRef(), MyGeometry, DragDropEvent );
	}
}

FReply SMultiBlockDragHandle::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FUICommandDragDropOp>().IsValid() )
	{
		BaseWidget.Pin()->OnCustomCommandDragged( Block.ToSharedRef(), MyGeometry, DragDropEvent );
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SMultiBlockDragHandle::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FUICommandDragDropOp>().IsValid() )
	{
		BaseWidget.Pin()->OnCustomCommandDropped();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SMultiBlockDragHandle::OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TSharedPtr<SWidget> CustomDecorator;

	TSharedRef<SWidget> BlockWidget = Block->MakeWidget(BaseWidget.Pin().ToSharedRef(), EMultiBlockLocation::None, false, nullptr)->AsWidget();

	if (Block->IsSeparator())
	{
		CustomDecorator = SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(SSeparator)
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
		
			+SHorizontalBox::Slot()
			.Padding( 20, 0, 40, 0 )
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromName(Block->GetExtensionHook()))
			]
		];
	}
	else if (Block->GetType() == EMultiBlockType::Heading)
	{
		CustomDecorator = SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(SSeparator)
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
		
			+SHorizontalBox::Slot()
			.Padding( 20, 0, 40, 0 )
			.AutoWidth()
			[
				BlockWidget
			]
		];
	}
	else
	{
		CustomDecorator = BlockWidget;
	}

	TSharedRef<FUICommandDragDropOp> NewOp = FUICommandDragDropOp::New(
			Block->GetExtensionHook(),
			Block->GetType(),
			Block->IsPartOfHeading(),
			NAME_None, 
			CustomDecorator,
		FVector2D(MyGeometry.AbsolutePosition)-MouseEvent.GetScreenSpacePosition()
		);

	NewOp->SetOnDropNotification( FSimpleDelegate::CreateSP( BaseWidget.Pin().ToSharedRef(), &SMultiBoxWidget::OnDropExternal ) );

	TSharedRef<FDragDropOperation> DragDropOp = NewOp;
	return FReply::Handled().BeginDragDrop( DragDropOp );
}

void SEditToolMenuDialog::InitMenu(UToolMenu* InMenu)
{
	MenuDialogOpenedWith = InMenu;

	OriginalSettings.Reset();
	if (UToolMenu* GeneratedMenu = UToolMenus::Get()->GenerateMenuOrSubMenuForEdit(InMenu))
	{
		for (const FCustomizedToolMenu* CustomizedToolMenu : GeneratedMenu->GetMenuCustomizationHierarchy().Hierarchy)
		{
			if (CustomizedToolMenu)
			{
				OriginalSettings.Add(*CustomizedToolMenu);
			}
		}
		CurrentGeneratedMenu = GeneratedMenu;
		MenuNames = GeneratedMenu->GetMenuHierarchyNames(true);
	}
	else
	{
		CurrentGeneratedMenu = nullptr;
		MenuNames.Reset();
	}
}

void SEditToolMenuDialog::Construct( const FArguments& InArgs )
{
	InitMenu(InArgs._SourceMenu);

	SetSelectedItem(NAME_None, ESelectedEditMenuEntryType::Menu);
	BuildWidget();
}

void SEditToolMenuDialog::HandleOnLiveCodingPatchComplete()
{
	Refresh();
}

void SEditToolMenuDialog::OnSelectedEntryChanged(TSharedRef<const FMultiBlock> InBlock)
{
	UToolMenu* ToolMenu = CurrentGeneratedMenu.Get();
	const FName BlockName = InBlock->GetExtensionHook();

	ESelectedEditMenuEntryType SelectedType = ESelectedEditMenuEntryType::None;
	if (InBlock->IsPartOfHeading())
	{
		SelectedType = ESelectedEditMenuEntryType::Section;
		if (!ToolMenu->ContainsSection(BlockName))
		{
			return;
		}
	}
	else
	{
		SelectedType = ESelectedEditMenuEntryType::Entry;
		if (!ToolMenu->ContainsEntry(BlockName))
		{
			return;
		}
	}

	SetSelectedItem(BlockName, SelectedType);
	if (PropertiesWidget.IsValid())
	{
		PropertiesWidget->SetObject(SelectedObject.Get());
	}
}

void SEditToolMenuDialog::SetSelectedItem(const FName InName, ESelectedEditMenuEntryType InType)
{
	UToolMenu* ToolMenu = CurrentGeneratedMenu.Get();
	if (!ToolMenu)
	{
		SelectedObject.Reset();
		return;
	}

	if (ESelectedEditMenuEntryType::Entry == InType)
	{
		UToolMenuEditorDialogEntry* NewSelectedObject = NewObject<UToolMenuEditorDialogEntry>();
		NewSelectedObject->Init(ToolMenu, InName);
		SelectedObject = TStrongObjectPtr<UToolMenuEditorDialogEntry>(NewSelectedObject);
	}
	else if (ESelectedEditMenuEntryType::Section == InType)
	{
		UToolMenuEditorDialogSection* NewSelectedObject = NewObject<UToolMenuEditorDialogSection>();
		NewSelectedObject->Init(ToolMenu, InName);
		SelectedObject = TStrongObjectPtr<UToolMenuEditorDialogSection>(NewSelectedObject);
	}
	else if (ESelectedEditMenuEntryType::Menu == InType)
	{
		UToolMenuEditorDialogMenu* NewSelectedObject = NewObject<UToolMenuEditorDialogMenu>();
		NewSelectedObject->Init(ToolMenu, ToolMenu->GetMenuName());
		SelectedObject = TStrongObjectPtr<UToolMenuEditorDialogMenu>(NewSelectedObject);
	}
	else
	{
		SelectedObject.Reset();
	}
}

TSharedRef<SWidget> SEditToolMenuDialog::ModifyBlockWidgetAfterMake(const TSharedRef<SMultiBoxWidget>& InMultiBoxWidget, const FMultiBlock& InBlock, const TSharedRef<SWidget>& InBlockWidget)
{
	TSharedRef<const FMultiBox> MultiBox = InMultiBoxWidget->GetMultiBox();

	const bool bIsEditing = MultiBox->IsInEditMode();

	static const FName SelectionColor("SelectionColor");

	UToolMenuBase* ToolMenuBase = MultiBox->GetToolMenu();
	const FName BlockName = InBlock.GetExtensionHook();
	const EMultiBlockType BlockType = InBlock.GetType();

	bool bIsBlockNamedAndToolMenu = false;
	if (BlockName != NAME_None && ToolMenuBase)
	{
		if (InBlock.IsPartOfHeading())
		{
			bIsBlockNamedAndToolMenu = ToolMenuBase->ContainsSection(BlockName);
		}
		else
		{
			bIsBlockNamedAndToolMenu = ToolMenuBase->ContainsEntry(BlockName);
		}
	}

	if (!bIsBlockNamedAndToolMenu)
	{
		return SNew(SOverlay)

		+SOverlay::Slot()
		[
			SNew(SHorizontalBox)
			
			+SHorizontalBox::Slot()
			.Padding(36, 0, 0, 0)
			[
				InBlockWidget
			]
		]
			
		+SOverlay::Slot()
		[
			SNew( SMultiBlockDragHandle, InMultiBoxWidget, InBlock.AsShared(), bIsBlockNamedAndToolMenu )
		];
	}
	else if (InBlock.IsSeparator() && InBlock.IsPartOfHeading())
	{
		return SNew(SOverlay)

		+SOverlay::Slot()
		[
			InBlockWidget
		]

		+SOverlay::Slot()
		[
			SNew( SMultiBlockDragHandle, InMultiBoxWidget, InBlock.AsShared(), bIsBlockNamedAndToolMenu )
		];
	}
	else
	{
		return SNew(SOverlay)

		+SOverlay::Slot()
		[
			SNew(SHorizontalBox)

			// Drag handle
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.Padding(4, 0, 32, 0)
					.AutoHeight()
					[
						SNew(SImage)
						.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
					]
				]
			]

			// Widget content
			+SHorizontalBox::Slot()
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					InBlockWidget
				]
			]
		]

		// Prevent input from reaching BlockWidget and allow drag/drop
		+SOverlay::Slot()
		[
			SNew( SMultiBlockDragHandle, InMultiBoxWidget, InBlock.AsShared(), bIsBlockNamedAndToolMenu )
		]

		// Toggle Hidden
		+SOverlay::Slot()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(20, 0, 0, 0)
				.AutoHeight()
				[
					// TODO: need to be able to update the proxy object being displayed in details panel
					// or better, perhaps have the proxy object displayed in the details panel directly lookup the config from UToolMenus singleton?
					SNew(SVisibilityEditWidget, InMultiBoxWidget, InBlock.AsShared(), SharedThis(this))
				]
			]
		];
	}
}

void SEditToolMenuDialog::BuildWidget()
{
	UToolMenu* ToolMenu = CurrentGeneratedMenu.Get();
	if (!ToolMenu)
	{
		ChildSlot
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "LargeText")
			.Text(LOCTEXT("Unavailable", "Unavailable"))
		];

		return;
	}
	
	ToolMenu->ModifyBlockWidgetAfterMake.BindSP(this, &SEditToolMenuDialog::ModifyBlockWidgetAfterMake);
	TSharedRef<SWidget> MenuWidget = UToolMenus::Get()->GenerateWidget(ToolMenu);

	TSharedRef<SMultiBoxWidget> MultiBoxWidget = StaticCastSharedRef<SMultiBoxWidget>(MenuWidget);
	TSharedRef<FMultiBox> MultiBox = ConstCastSharedRef<FMultiBox>(MultiBoxWidget->GetMultiBox());
	MultiBox->OnEditSelectionChanged().BindSP(this, &SEditToolMenuDialog::OnSelectedEntryChanged);

	TSharedRef<SWidget> DialogCenter =
	
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			// Preview
			SNew(SScrollBox)

			+SScrollBox::Slot()
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 8, 0, 8)
				[
					MenuWidget
				]
			]
		]

		+SHorizontalBox::Slot()
		.Padding(32, 0, 0, 0)
		[
			SNew(SVerticalBox)

			// Dialog buttons Save/Cancel
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0)
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
								
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("UndoAllChangesButtonText", "Undo all changes"))
					.ToolTipText(LOCTEXT("UndoAllChangesButtonTooltip", "Undos all changes made to these menus"))
					.OnClicked(this, &SEditToolMenuDialog::UndoAllChanges)
				]

				+SHorizontalBox::Slot()
				.Padding(8, 0, 0, 0)
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("ResetAllDefaultsButtonText", "Reset all to Defaults"))
					.ToolTipText(LOCTEXT("ResetAllDefaultsButtonTooltip", "Reset all settings for all menus to default values"))
					.OnClicked(this, &SEditToolMenuDialog::HandleResetAllClicked)
				]

				+SHorizontalBox::Slot()
				.Padding(8, 0, 0, 0)
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("ResetDefaultsButtonText", "Reset to Defaults"))
					.ToolTipText(LOCTEXT("ResetDefaultsButtonTooltip", "Reset settings for current menu to default values"))
					.OnClicked(this, &SEditToolMenuDialog::HandleResetClicked)
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				BuildMenuPropertiesWidget()
			]
		];

	{
		MenuNameComboData.Reset();
		for (FName It : MenuNames)
		{
			MenuNameComboData.Add(MakeShareable(new FName(It)));
		}
	}

	ChildSlot
	[
		SNew(SBorder)
		.Padding(20)
		.BorderImage( FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush") )
		[
			SNew(SVerticalBox)

			// Title
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "LargeText")
				.Text(FText::FromName(ToolMenu->MenuName))
			]

			// Menu names
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0)
			[
				SNew(SComboBox< TSharedPtr<FName> >)
				.OptionsSource(&MenuNameComboData)
				.OnGenerateWidget(this, &SEditToolMenuDialog::MakeMenuNameComboEntryWidget)
				.OnSelectionChanged(this, &SEditToolMenuDialog::OnMenuNamesSelectionChanged)
				[
					SNew(STextBlock)
					.Text(FText::FromName(ToolMenu->MenuName))
				]
			]

			// Title spacer
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 16, 0, 8)
			[
				SNew(SSeparator)
			]

			// Middle of dialog
			+SVerticalBox::Slot()
			[
				DialogCenter
			]
		]
	];
}

TSharedRef<SWidget> SEditToolMenuDialog::BuildMenuPropertiesWidget()
{
	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertiesWidget = PropertyModule.CreateDetailView(Args);
	PropertiesWidget->SetObject(SelectedObject.Get());

	return PropertiesWidget.ToSharedRef();
}

TSharedRef<SWidget> SEditToolMenuDialog::MakeMenuNameComboEntryWidget(TSharedPtr<FName> InEntry)
{
	return SNew(STextBlock)
			.Text(FText::FromName(*InEntry));
}

void SEditToolMenuDialog::OnMenuNamesSelectionChanged(TSharedPtr<FName> InEntry, ESelectInfo::Type SelectInfo)
{
	UToolMenu* SourceMenu = MenuDialogOpenedWith.Get();
	if (!SourceMenu)
	{
		BuildWidget();
		return;
	}

	if (!InEntry.IsValid())
	{
		BuildWidget();
		return;
	}

	const FName* NewMenuNamePtr = InEntry.Get();
	const FName NewMenuName = NewMenuNamePtr ? *NewMenuNamePtr : NAME_None;

	FName OldMenuName;
	if (UToolMenu* ToolMenu = CurrentGeneratedMenu.Get())
	{
		if (ToolMenu->GetMenuName() == NewMenuName)
		{
			return;
		}

		OldMenuName = ToolMenu->GetMenuName();
	}
	
	FToolMenuContext NewMenuContext = SourceMenu->Context;
	NewMenuContext.SetIsEditing(true);
	
	TArray<const UToolMenu*> SubMenuChain = SourceMenu->GetSubMenuChain();
	if (!SourceMenu->SubMenuParent)
	{
		CurrentGeneratedMenu = UToolMenus::Get()->GenerateMenu(NewMenuName, NewMenuContext);
	}
	else if (SubMenuChain.Num() > 0)
	{
		FString SubMenuNamePath = SourceMenu->GetSubMenuNamePath();

		FString NewMenuNameString = NewMenuName.ToString();
		int32 NumCharactersToKeep = NewMenuNameString.Len() - SubMenuNamePath.Len() - 1;
		if (NumCharactersToKeep > 0)
		{
			// "Menu" and "SubMenuA"
			FString BaseMenuPath = NewMenuNameString.Left(NumCharactersToKeep);

			// Create the base menu, then walk the down the SubMenus
			if (UToolMenu* NewMenu = UToolMenus::Get()->GenerateMenu(*BaseMenuPath, NewMenuContext))
			{
				for (int32 i=1; i < SubMenuChain.Num(); ++i)
				{
					if (UToolMenu* SubMenu = UToolMenus::Get()->GenerateSubMenu(NewMenu, SubMenuChain[i]->SubMenuSourceEntryName))
					{
						NewMenu = SubMenu;
					}
					else
					{
						NewMenu = nullptr;
						break;
					}
				}

				if (NewMenu)
				{
					CurrentGeneratedMenu = NewMenu;
				}
			}
		}
	}

	BuildWidget();
}

void SEditToolMenuDialog::OnToggleVisibleClicked(TSharedRef<const FMultiBlock> InBlock, TWeakPtr<SMultiBoxWidget> BaseWidget)
{
	const FName ItemName = InBlock->GetExtensionHook();
	if (ItemName == NAME_None)
	{
		return;
	}

	UToolMenuBase* ToolMenu = BaseWidget.IsValid() ? BaseWidget.Pin()->GetMultiBox()->GetToolMenu() : nullptr;
	if (!ToolMenu)
	{
		return;
	}

	// Commit change
	FCustomizedToolMenu* MenuCustomization = ToolMenu->AddMenuCustomization();
	FCustomizedToolMenuHierarchy MenuCustomizationHierarchy = ToolMenu->GetMenuCustomizationHierarchy();
	if (InBlock->GetType() == EMultiBlockType::Heading)
	{
		bool bHidden = MenuCustomizationHierarchy.IsSectionHidden(ItemName);
		MenuCustomization->AddSection(ItemName)->Visibility = bHidden ? ECustomizedToolMenuVisibility::Visible : ECustomizedToolMenuVisibility::Hidden;
	}
	else
	{
		bool bHidden = MenuCustomizationHierarchy.IsEntryHidden(ItemName);
		MenuCustomization->AddEntry(ItemName)->Visibility = bHidden ? ECustomizedToolMenuVisibility::Visible : ECustomizedToolMenuVisibility::Hidden;
	}

	LoadSelectedObjectState();
}

void SEditToolMenuDialog::LoadSelectedObjectState()
{
	if (SelectedObject.IsValid())
	{
		SelectedObject->LoadState();
		SelectedObject->PostEditChange();
		if (PropertiesWidget.IsValid())
		{
			PropertiesWidget->ForceRefresh();
		}
	}
}

void SEditToolMenuDialog::CloseContainingWindow()
{
	TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}
}

FReply SEditToolMenuDialog::Refresh()
{
	if (UToolMenu* ToolMenu = CurrentGeneratedMenu.Get())
	{
		CurrentGeneratedMenu = UToolMenus::Get()->GenerateMenuOrSubMenuForEdit(ToolMenu);
		UToolMenus::Get()->RefreshMenuWidget(ToolMenu->GetMenuName());
	}

	BuildWidget();

	LoadSelectedObjectState();

	return FReply::Handled();
}

FReply SEditToolMenuDialog::HandleResetClicked()
{
	if (UToolMenu* ToolMenu = CurrentGeneratedMenu.Get())
	{
		UToolMenus::Get()->RemoveCustomization(ToolMenu->GetMenuName());
		Refresh();
	}

	return FReply::Handled();
}

FReply SEditToolMenuDialog::HandleResetAllClicked()
{
	if (FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::No, LOCTEXT("ResetAllQuestion", "Remove all menu customizations for all menus?"), LOCTEXT("ResetAllQuestion_Question", "Question")) == EAppReturnType::Yes)
	{
		UToolMenus::Get()->RemoveAllCustomizations();
		OriginalSettings.Reset();
		Refresh();
	}

	return FReply::Handled();
}

void SEditToolMenuDialog::SaveSettingsToDisk()
{
	UToolMenus::Get()->SaveCustomizations();
}

FReply SEditToolMenuDialog::UndoAllChanges()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	for (const FName& HierarchyMenuName : MenuNames)
	{
		ToolMenus->RemoveCustomization(HierarchyMenuName);
	}

	for (const FCustomizedToolMenu& Original : OriginalSettings)
	{
		FCustomizedToolMenu* Destination = ToolMenus->AddMenuCustomization(Original.Name);
		*Destination = Original;
	}

	Refresh();
	return FReply::Handled();
}

void SEditToolMenuDialog::OnWindowClosed(const TSharedRef<SWindow>& Window)
{	
	SaveSettingsToDisk();
	UToolMenus::Get()->RefreshAllWidgets();
}

#undef LOCTEXT_NAMESPACE
