// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SComboBox.h"
#include "ToolMenus.h"
#include "UObject/StrongObjectPtr.h"
#include "ToolMenusEditor.h"


class UToolMenuEditorDialogEntry;


/**
 * A dialog to customize a menu
 */
class SEditToolMenuDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SEditToolMenuDialog )
	{}

	/** A reference to the parent window */
	SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)

	SLATE_ARGUMENT(UToolMenu*, SourceMenu)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );
	
	void OnToggleVisibleClicked(TSharedRef<const FMultiBlock> InBlock, TWeakPtr<SMultiBoxWidget> BaseWidget);
	
	/** Closes the window that contains this widget */
	void CloseContainingWindow();

	void OnWindowClosed(const TSharedRef<SWindow>& Window);

private:

	TSharedRef<SWidget> ModifyBlockWidgetAfterMake(const TSharedRef<SMultiBoxWidget>& InMultiBoxWidget, const FMultiBlock& InBlock, const TSharedRef<SWidget>& InBlockWidget);

	void BuildWidget();
	void HandleOnLiveCodingPatchComplete();

	void InitMenu(UToolMenu* InMenu);

	FReply Refresh();
	FReply HandleResetClicked();
	FReply HandleResetAllClicked();
	FReply UndoAllChanges();

	void LoadSelectedObjectState();
	void SaveSettingsToDisk();

	TSharedRef<SWidget> BuildMenuPropertiesWidget();
	TSharedRef<SWidget> MakeMenuNameComboEntryWidget(TSharedPtr<FName> InEntry);
	void OnMenuNamesSelectionChanged(TSharedPtr<FName> InEntry, ESelectInfo::Type SelectInfo);

	void OnSelectedEntryChanged(TSharedRef<const FMultiBlock> InBlock);

	void SetSelectedItem(const FName InName, ESelectedEditMenuEntryType InType);

	TWeakObjectPtr<UToolMenu> MenuDialogOpenedWith;
	TWeakObjectPtr<UToolMenu> CurrentGeneratedMenu;

	TArray<FName> MenuNames;
	TArray<TSharedPtr<FName>> MenuNameComboData;

	TSharedPtr<class IDetailsView> PropertiesWidget;
	TStrongObjectPtr<UToolMenuEditorDialogObject> SelectedObject;

	TArray<FCustomizedToolMenu> OriginalSettings;
};
