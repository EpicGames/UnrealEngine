// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenusEditor.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolMenusEditor)

void UToolMenuEditorDialogMenu::Init(UToolMenu* InMenu, const FName InName)
{
	Menu = InMenu;
	Name = InName;

	LoadState();
}

void UToolMenuEditorDialogSection::Init(UToolMenu* InMenu, const FName InName)
{
	Name = InName;
	Type = ESelectedEditMenuEntryType::Section;
	Menu = InMenu;

	LoadState();
}

void UToolMenuEditorDialogSection::LoadState()
{
	Super::LoadState();

	Visibility = ECustomizedToolMenuVisibility::None;

	if (Menu && Name != NAME_None)
	{
		if (FCustomizedToolMenu* CustomizedToolMenu = Menu->FindMenuCustomization())
		{
			if (FCustomizedToolMenuSection* Found = CustomizedToolMenu->Sections.Find(Name))
			{
				Visibility = Found->Visibility;
			}
		}
	}
}

void UToolMenuEditorDialogSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UToolMenuEditorDialogEntry, Visibility))
	{
		if (Menu)
		{
			FCustomizedToolMenu* CustomizedToolMenu = Menu->AddMenuCustomization();
			CustomizedToolMenu->AddSection(Name)->Visibility = Visibility;
		}
	}
}

void UToolMenuEditorDialogEntry::Init(UToolMenu* InMenu, const FName InName)
{
	Name = InName;
	Type = ESelectedEditMenuEntryType::Entry;
	Menu = InMenu;

	OwnerName = TEXT("");
	ScriptObject = TEXT("");
	ScriptObjectClass = TEXT("");

	if (Menu)
	{
		for (const FToolMenuSection& Section : Menu->Sections)
		{
			if (const FToolMenuEntry* FoundEntry = Section.FindEntry(Name))
			{
				if (FoundEntry->Owner.IsSet())
				{
					if (!FoundEntry->Owner.TryGetName().IsNone())
					{
						OwnerName = FoundEntry->Owner.TryGetName().ToString();
					}
					else
					{
						OwnerName = TEXT("<Pointer>");
					}
				}

				if (UToolMenuEntryScript* EntryScriptObject = FoundEntry->ScriptObject)
				{
					ScriptObject = EntryScriptObject->GetFullName();
					ScriptObjectClass = EntryScriptObject->GetClass()->GetFullName();
				}

				break;
			}
		}
	}

	LoadState();
}

void UToolMenuEditorDialogEntry::LoadState()
{
	Super::LoadState();

	Visibility = ECustomizedToolMenuVisibility::None;

	if (Menu && Name != NAME_None)
	{
		if (FCustomizedToolMenu* CustomizedToolMenu = Menu->FindMenuCustomization())
		{
			if (FCustomizedToolMenuEntry* Found = CustomizedToolMenu->Entries.Find(Name))
			{
				Visibility = Found->Visibility;
			}
		}
	}
}

void UToolMenuEditorDialogEntry::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UToolMenuEditorDialogEntry, Visibility))
	{
		if (Menu)
		{
			FCustomizedToolMenu* CustomizedToolMenu = Menu->AddMenuCustomization();
			CustomizedToolMenu->AddEntry(Name)->Visibility = Visibility;
		}
	}
}
