// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenusEditor.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IToolMenusEditorModule.h"
#include "SEditToolMenuDialog.h"
#include "ToolMenus.h"
#include "Framework/Docking/TabManager.h"

#define LOCTEXT_NAMESPACE "ToolMenusEditor"

static bool ToolMenusEditWindowModalCVar = true;
FAutoConsoleVariableRef CVarRex(
	TEXT("ToolMenus.EditWindowModal"),
	ToolMenusEditWindowModalCVar,
	TEXT("Edit menus in modal window"),
	ECVF_Default);

/**
 * Implements the Tool menus module.
 */
class FToolMenusEditorModule
	: public IToolMenusEditorModule
{
public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}

	// End IModuleInterface interface

	virtual void RegisterShowEditMenusModeCheckbox() const override
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Window");
		FToolMenuSection& Section = Menu->FindOrAddSection("WindowLayout");

		if (!Section.FindEntry("EnableMenuEditing"))
		{
			Section.AddMenuEntry(
				"EnableMenuEditing",
				LOCTEXT("EnableMenuEditing", "Enable Menu Editing"),
				LOCTEXT("EnableMenuEditing_ToolTip", "Adds command to each menu and toolbar for editing"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([]()
					{
						UToolMenus* ToolMenus = UToolMenus::Get();
						ToolMenus->SetEditMenusMode(!ToolMenus->GetEditMenusMode());
					}),
					FCanExecuteAction(),
					FGetActionCheckState::CreateLambda([]()
					{
						return UToolMenus::Get()->GetEditMenusMode() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; 
					})),
				EUserInterfaceActionType::ToggleButton
			);
		}
	}

	virtual void OpenEditToolMenuDialog(UToolMenu* ToolMenu) const
	{
		TSharedPtr<SWindow> ParentWindow = FGlobalTabmanager::Get()->GetRootWindow();
		const FVector2D WindowSize = FVector2D(940, 540);
		FText WindowTitle = LOCTEXT("EditMenu_Title", "Edit Menu");

		TSharedRef<SWindow> EditMenuWindow =
			SNew(SWindow)
			.Title(WindowTitle)
			.ClientSize(WindowSize)
			.SupportsMinimize(true)
			.SupportsMaximize(true)
			.MinWidth(620)
			.MinHeight(260);

		TSharedRef<SEditToolMenuDialog> NewDialog =
			SNew(SEditToolMenuDialog)
			.ParentWindow(EditMenuWindow)
			.SourceMenu(ToolMenu);

		EditMenuWindow->SetContent(NewDialog);

		EditMenuWindow->SetOnWindowClosed(FOnWindowClosed::CreateSP(NewDialog, &SEditToolMenuDialog::OnWindowClosed));

		if (ToolMenusEditWindowModalCVar)
		{
			FSlateApplication::Get().AddModalWindow(EditMenuWindow, ParentWindow);
		}
		else
		{
			FSlateApplication::Get().AddWindow(EditMenuWindow);
		}
	}
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FToolMenusEditorModule, ToolMenusEditor)

