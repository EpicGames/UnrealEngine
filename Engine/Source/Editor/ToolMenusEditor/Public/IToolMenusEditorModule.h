// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogToolMenusEditor, Log, All);

class IToolMenusEditorModule : public IModuleInterface
{
public:

	/**
	 * Retrieve the module instance.
	 */
	static inline IToolMenusEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IToolMenusEditorModule>("ToolMenusEditor");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("ToolMenusEditor");
	}

	/**
	 * Open editor for the given ToolMenu
	 *
	 * @param ToolMenu The menu top edit
	 */
	virtual void OpenEditToolMenuDialog(class UToolMenu* ToolMenu) const = 0;

	/**
	 * Adds menu entry to toggle adding an entry to each menu and toolbar to open a menu editor
	 */
	virtual void RegisterShowEditMenusModeCheckbox() const = 0;
};

