// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "ToolMenus.h"

#include "ToolMenusEditor.generated.h"

UENUM()
enum class ESelectedEditMenuEntryType : uint8
{
	None = 0,
	Entry,
	Section,
	Menu
};

UCLASS(Abstract)
class UToolMenuEditorDialogObject : public UObject
{
	GENERATED_BODY()

public:

	virtual void LoadState() {}
};

UCLASS()
class UToolMenuEditorDialogMenu : public UToolMenuEditorDialogObject
{
	GENERATED_BODY()

public:

	void Init(class UToolMenu* InMenu, const FName InName);

	UPROPERTY(VisibleAnywhere, Category = Misc)
	FName Name;

	UPROPERTY()
	TObjectPtr<UToolMenu> Menu;
};

UCLASS()
class UToolMenuEditorDialogBlock : public UToolMenuEditorDialogObject
{
	GENERATED_BODY()

public:

	UPROPERTY(VisibleAnywhere, Category = Misc)
	FName Name;

	UPROPERTY(VisibleAnywhere, Category = Misc)
	ESelectedEditMenuEntryType Type;

	UPROPERTY(EditAnywhere, Category = Misc)
	ECustomizedToolMenuVisibility Visibility;

	UPROPERTY(VisibleAnywhere, Category = Misc)
	FString OwnerName;

	UPROPERTY(VisibleAnywhere, Category = Misc)
	FString ScriptObject;

	UPROPERTY(VisibleAnywhere, Category = Misc)
	FString ScriptObjectClass;

	UPROPERTY()
	TObjectPtr<UToolMenu> Menu;
};

UCLASS()
class UToolMenuEditorDialogSection : public UToolMenuEditorDialogBlock
{
	GENERATED_BODY()

public:

	void Init(class UToolMenu* InMenu, const FName InName);

	virtual void LoadState() override;

	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
};

UCLASS()
class UToolMenuEditorDialogEntry : public UToolMenuEditorDialogBlock
{
	GENERATED_BODY()

public:

	void Init(class UToolMenu* InMenu, const FName InName);

	virtual void LoadState() override;

	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
};
