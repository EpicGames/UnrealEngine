// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class ToolMenusEditor : ModuleRules
	{
		public ToolMenusEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"ToolMenus",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Slate",
					"SlateCore",
					"EditorFramework",
					"UnrealEd",
					"InputCore",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"AssetRegistry"
			});

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"AssetRegistry"
				});
		}
	}
}
